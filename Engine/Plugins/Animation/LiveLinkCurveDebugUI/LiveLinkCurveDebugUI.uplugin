{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Live Link Curve Debug UI",
	"Description": "Allows Viewing LiveLink Curve Debug Information",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": true,
	"Modules": [
		{
			"Name": "LiveLinkCurveDebugUI",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		}
	],
	"Plugins": [
		{
			"Name": "LiveLink",
			"Enabled": true
		}
	]
}