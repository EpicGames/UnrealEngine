// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class LiveLinkCurveDebugUI : ModuleRules
	{
		public LiveLinkCurveDebugUI(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"LiveLinkInterface",
					"Slate",
					"SlateCore",
					"InputCore"
				}
			);

			PrivateIncludePathModuleNames.AddRange(
				new string[]
				{
					"WorkspaceMenuStructure"
				}
			);

			// DesktopPlatform is only available for Editor and Program targets (running on a desktop platform)
			bool IsDesktopPlatformType = Target.Platform == UnrealBuildTool.UnrealTargetPlatform.Win64
				|| Target.Platform == UnrealBuildTool.UnrealTargetPlatform.Mac
				|| Target.Platform == UnrealBuildTool.UnrealTargetPlatform.Linux;
			if (Target.Type == TargetType.Editor || (Target.Type == TargetType.Program && IsDesktopPlatformType))
			{
				PublicDefinitions.Add("LIVELINK_CURVE_DEBUG_UI_HAS_DESKTOP_PLATFORM=1");
			}
			else
			{
				PublicDefinitions.Add("LIVELINK_CURVE_DEBUG_UI_HAS_DESKTOP_PLATFORM=0");
			}

			if (Target.bBuildEditor == true)
			{
                PrivateDependencyModuleNames.Add("EditorFramework");
				PrivateDependencyModuleNames.Add("UnrealEd");
			}
		}
	}
}
