// Copyright Epic Games, Inc. All Rights Reserved.

//General Includes
#include "LiveLinkCurveDebugUIModule.h"
#include "LiveLinkCurveDebugPrivate.h"

//Tab Manager Support

//General Slate Widget Support

//Actual Debugger Widgets
#include "SLiveLinkCurveDebugUI.h"
#include "SLiveLinkCurveDebugUITab.h"

//Settings Object
#include "LiveLinkDebuggerSettings.h"

//Engine imports
#include "Engine/GameViewportClient.h"

#if WITH_EDITOR
#include "ISettingsModule.h"
#include "Editor/EditorEngine.h"

//Needed to register with Developer Tools Windows Menu
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#else
#include "Engine/Engine.h"
#include "Engine/World.h"
#endif

#define LOCTEXT_NAMESPACE "FLiveLinkCurveDebugUIModule"

bool FLiveLinkCurveDebugUIModule::Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (FParse::Command(&Cmd, TEXT("LiveLinkDebugger")))
	{
		FString ParsedCommand = FParse::Token(Cmd, 0);

		if (ParsedCommand.Equals(TEXT("Show"),ESearchCase::IgnoreCase))
		{
			FString ParsedSubjectName = FParse::Token(Cmd, 0);

			if (ParsedSubjectName.Equals(TEXT("Next"), ESearchCase::IgnoreCase))
			{
				SwitchToNextLiveLinkSubject();
				Ar.Log(ELogVerbosity::Display, TEXT("Switching to using next available LiveLinkSubjectName!"));
			}
			else
			{
				DisplayLiveLinkCurveDebugUI(ParsedSubjectName);

				if (ParsedSubjectName.IsEmpty())
				{
					Ar.Log(ELogVerbosity::Display, TEXT("Displaying LiveLinkDebugger. No LiveLinkSubject Name Supplied. Using First Available."));
				}
				else
				{
					Ar.Log(ELogVerbosity::Display, FString::Printf(TEXT("Displaying LiveLinkDebugger with SubjectName: %s"), *ParsedSubjectName));
				}
			}
		}
		else if (ParsedCommand.Equals(TEXT("AddViewport"), ESearchCase::IgnoreCase))
		{
			FString NoSubjectName;

			bForceDisplayThroughViewport = true;
			DisplayLiveLinkCurveDebugUI(NoSubjectName);
			bForceDisplayThroughViewport = false;

			Ar.Log(ELogVerbosity::Display, TEXT("Forcing LiveLinkDebugger to Display Through Viewport"));
		}
		else if (ParsedCommand.Equals(TEXT("Hide"),ESearchCase::IgnoreCase))
		{
			HideLiveLinkCurveDebugUI();
			Ar.Log(ELogVerbosity::Display, TEXT("LiveLinkDebugger: Hiding Widget."));
		}
		else if (ParsedCommand.Equals(TEXT("Next"), ESearchCase::IgnoreCase))
		{
			SwitchToNextLiveLinkSubject();
			Ar.Log(ELogVerbosity::Display, TEXT("Switching to using next available LiveLinkSubjectName!"));
		}
		else
		{
			Ar.Log(ELogVerbosity::Display, TEXT("LiveLinkDebugger: Unrecognized command."));
		}

		return true;
	}

	return false;
}

void FLiveLinkCurveDebugUIModule::DisplayLiveLinkCurveDebugUI(FString& LiveLinkSubjectName)
{
	UE_LOGF(LogLiveLinkCurveDebugUI, Display, "Displaying LiveLinkCurveDebugUI for %ls", *LiveLinkSubjectName);

	//Cache off the info we need to track in our UI
	LiveLinkSubjectNameToTrack = *LiveLinkSubjectName;

#if LIVELINK_CURVE_DEBUG_UI_HAS_DESKTOP_PLATFORM
	if (bForceDisplayThroughViewport)
	{
		DisplayThroughViewportAdd();
	}
	else
	{
		DisplayThroughTab();
	}
#else
	DisplayThroughViewportAdd();
#endif
}

void FLiveLinkCurveDebugUIModule::HideLiveLinkCurveDebugUI()
{
	UE_LOGF(LogLiveLinkCurveDebugUI, Display, "Attempting to remove LiveLinkCurveDebugUI from viewport.");

	if (LiveLinkUserWidget.IsValid())
	{
		RemoveWidgetFromViewport();
		LiveLinkUserWidget.Reset();
	}
}

void FLiveLinkCurveDebugUIModule::RegisterTabSpawner()
{
	if (bHasRegisteredTabSpawners)
	{
		UnregisterTabSpawner();
	}

	FTabSpawnerEntry& SpawnerEntry = FGlobalTabmanager::Get()->RegisterNomadTabSpawner("LiveLinkCurveDebug", FOnSpawnTab::CreateRaw(this, &FLiveLinkCurveDebugUIModule::MakeLiveLinkCurveDebugTab))
		.SetDisplayName(LOCTEXT("LiveLinkCurveDebugTitle", "Live Link Curve Debugger"))
		.SetTooltipText(LOCTEXT("LiveLinkCurveDebugTooltipText", "Open the Live Link Curve Debugger tab."))
		.SetIcon(FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "WidgetReflector.TabIcon")); // @TODO - give this its own icon

	bHasRegisteredTabSpawners = true;

	//Register with the Developer Tools Menu
#if WITH_EDITOR
	const IWorkspaceMenuStructure& MenuStructure = WorkspaceMenu::GetMenuStructure();
	SpawnerEntry.SetGroup(MenuStructure.GetDeveloperToolsDebugCategory());
#endif //WITH_EDITOR
}

void FLiveLinkCurveDebugUIModule::UnregisterTabSpawner()
{
	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner("LiveLinkCurveDebug");
	bHasRegisteredTabSpawners = false;
}

TSharedRef<SDockTab> FLiveLinkCurveDebugUIModule::MakeLiveLinkCurveDebugTab(const FSpawnTabArgs& Args)
{
	TSharedPtr<SDockTab> TabLiveLinkDebugUI = SNew(SLiveLinkCurveDebugUITab)
		.InitialLiveLinkSubjectName(*LiveLinkSubjectNameToTrack);

	return TabLiveLinkDebugUI.ToSharedRef();
}

void FLiveLinkCurveDebugUIModule::StartupModule()
{
	bForceDisplayThroughViewport = false;

	bHasRegisteredTabSpawners = false;
	RegisterTabSpawner();
}

void FLiveLinkCurveDebugUIModule::ShutdownModule()
{
	HideLiveLinkCurveDebugUI();
	UnregisterTabSpawner();
}

void FLiveLinkCurveDebugUIModule::DisplayThroughTab()
{
	check(bHasRegisteredTabSpawners);
	FGlobalTabmanager::Get()->TryInvokeTab(FTabId("LiveLinkCurveDebug"));
}

TSharedPtr<SLiveLinkCurveDebugUI> FLiveLinkCurveDebugUIModule::CreateDebugWidget()
{
	float DPIScale = GetDPIScaleFromSettings();

	return SNew(SLiveLinkCurveDebugUI)
		.DPIScale(DPIScale);
}

float FLiveLinkCurveDebugUIModule::GetDPIScaleFromSettings()
{
	float ReturnedDPI = 1.0f;

	//Attempt to pull DPI from LiveLinkDebuggerSettings using our Viewport Size
	const ULiveLinkDebuggerSettings* UISettings = GetDefault<ULiveLinkDebuggerSettings>(ULiveLinkDebuggerSettings::StaticClass());
    const UGameViewportClient* GameViewport = GetGameViewportClientForDebugUIModule();
	if (GameViewport && UISettings)
	{
		FVector2D ViewportSize;
        GameViewport->GetViewportSize(ViewportSize);

		ReturnedDPI = UISettings->GetDPIScaleBasedOnSize(FIntPoint(ViewportSize.X, ViewportSize.Y));
	}

	return ReturnedDPI;
}

void FLiveLinkCurveDebugUIModule::DisplayThroughViewportAdd()
{
	//If we are adding a new Viewport Widget, hide any existing ones first
	HideLiveLinkCurveDebugUI();

	LiveLinkUserWidget = CreateDebugWidget();
	if (LiveLinkUserWidget.IsValid())
	{
		LiveLinkUserWidget->SetLiveLinkSubjectName(*LiveLinkSubjectNameToTrack);
		
		if (!AddWidgetToViewport())
		{
			UE_LOGF(LogLiveLinkCurveDebugUI, Warning, "Unable to add LiveLinkCurveDebug User Widget to the viewport!");
		}
	}
}

bool FLiveLinkCurveDebugUIModule::AddWidgetToViewport()
{
    UGameViewportClient* GameViewport = GetGameViewportClientForDebugUIModule();
	if (GameViewport)
	{
		//Using a ZOrder of INDEX_NONE as this causes it to get added on top of all other active widgets
		const int32 ZOrder = INDEX_NONE;
        
        UWorld* World = GetWorldForDebugUIModule();
        
        //Prioritize adding it through the AddViewportWidgetForPlayer function
        if (World)
        {
            ULocalPlayer* LP = World->GetFirstLocalPlayerFromController();
            GameViewport->AddViewportWidgetForPlayer(LP, LiveLinkUserWidget.ToSharedRef(), ZOrder);
        }
        //No World, just add it through the general widget content function
        else
        {
            GameViewport->AddViewportWidgetContent(LiveLinkUserWidget.ToSharedRef(),ZOrder);
        }
		return true;
	}

	return false;
}

void FLiveLinkCurveDebugUIModule::RemoveWidgetFromViewport()
{
	if (LiveLinkUserWidget.IsValid())
	{
        UGameViewportClient* GameViewport = GetGameViewportClientForDebugUIModule();
		if (GameViewport)
		{
            UWorld* World = GetWorldForDebugUIModule();
            
            if (World)
            {
                ULocalPlayer* LP = World->GetFirstLocalPlayerFromController();
                GameViewport->RemoveViewportWidgetForPlayer(LP, LiveLinkUserWidget.ToSharedRef());
                
                UE_LOGF(LogLiveLinkCurveDebugUI, Display, "Successfully removed LiveLinkUserWidget from Player's Viewport!");
            }
            else
            {
                GameViewport->RemoveViewportWidgetContent(LiveLinkUserWidget.ToSharedRef());
                
                UE_LOGF(LogLiveLinkCurveDebugUI, Display, "Successfully removed LiveLinkUserWidget from Viewport! (No Player Supplied)");
            }
		}
	}
}

UWorld* FLiveLinkCurveDebugUIModule::GetWorldForDebugUIModule()
{
	UWorld* World = nullptr;

#if WITH_EDITOR
	UEditorEngine* EEngine = Cast<UEditorEngine>(GEngine);
	if (GIsEditor && EEngine != nullptr)
	{
		// lets use PlayWorld during PIE/Simulate and regular world from editor otherwise, to draw debug information
		World = EEngine->PlayWorld != nullptr ? ToRawPtr(EEngine->PlayWorld) : EEngine->GetEditorWorldContext().World();
	}
#endif

	if (!GIsEditor && World == nullptr && GEngine != nullptr)
	{
		World = GEngine->GetWorld();
	}

	return World;
}

UGameViewportClient* FLiveLinkCurveDebugUIModule::GetGameViewportClientForDebugUIModule()
{
    UGameViewportClient* ReturnedViewport = nullptr;
    
    UWorld* World = GetWorldForDebugUIModule();
    if (World)
    {
        ReturnedViewport = World->GetGameViewport();
    }
    else if (GEngine)
    {
        ReturnedViewport = GEngine->GameViewport;
    }
    
    return ReturnedViewport;
}

void FLiveLinkCurveDebugUIModule::SwitchToNextLiveLinkSubject()
{
	if (LiveLinkUserWidget.IsValid())
	{
		LiveLinkUserWidget->ChangeToNextValidLiveLinkSubjectName();
	}
}

void FLiveLinkCurveDebugUIModule::RegisterSettings()
{
#if WITH_EDITOR
    if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
    {
        SettingsModule->RegisterSettings("Project", "Plugins", "LiveLinkCurveDebugger",
                                         LOCTEXT("RuntimeSettingsName", "Live Link Curve Debugger"),
                                         LOCTEXT("RuntimeSettingsDescription", "Configure the Live Link Curve Debugger Plugin"),
                                         GetMutableDefault<ULiveLinkDebuggerSettings>());
    }
#endif //WITH_EDITOR
}

void FLiveLinkCurveDebugUIModule::UnRegisterSettings()
{
#if WITH_EDITOR
    if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
    {
        SettingsModule->UnregisterSettings("Project", "Plugins", "LiveLinkCurveDebugger");
    }
#endif //WITH_EDITOR
}
#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FLiveLinkCurveDebugUIModule, LiveLinkCurveDebugUI);
DEFINE_LOG_CATEGORY(LogLiveLinkCurveDebugUI);
