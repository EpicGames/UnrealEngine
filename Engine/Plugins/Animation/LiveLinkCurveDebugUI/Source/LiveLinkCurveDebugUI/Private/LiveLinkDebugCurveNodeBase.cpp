// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkDebugCurveNodeBase.h"

#include "LiveLinkDebuggerSettings.h"

	FLiveLinkDebugCurveNodeBase::FLiveLinkDebugCurveNodeBase(FName InCurveName, float InCurveValue)
		: CurveName(InCurveName)
		, CurveValue(InCurveValue)
	{}

	FSlateColor FLiveLinkDebugCurveNodeBase::GetCurveFillColor() const
	{
		const ULiveLinkDebuggerSettings* UISettings = GetDefault<ULiveLinkDebuggerSettings>(ULiveLinkDebuggerSettings::StaticClass());
		if (UISettings)
		{
			return UISettings->GetBarColorForCurveValue(CurveValue);
		}

		return FSlateColor(FLinearColor::Red);
	}
