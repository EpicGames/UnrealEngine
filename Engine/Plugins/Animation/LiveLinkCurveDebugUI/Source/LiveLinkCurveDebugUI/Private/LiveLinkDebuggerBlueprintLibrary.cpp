// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkDebuggerBlueprintLibrary.h"

#include "Engine/Engine.h"

#include "Modules/ModuleManager.h"

#include "ILiveLinkCurveDebugUIModule.h"

void PrintErrorToScreen(const FString& ErrorMessage)
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	GEngine->AddOnScreenDebugMessage(INDEX_NONE, 3600.0f, FColor(255, 48, 16), *ErrorMessage);
#endif //!(UE_BUILD_SHIPPING || UE_BUILD_TEST)
}

ILiveLinkCurveDebugUIModule* GetLiveLinkDebugModule()
{
	static const FString NoValidARCurveDebugModule_Warning = FString(TEXT("No valid ILiveLinkCurveDebugUIModule module loaded!"));

	ILiveLinkCurveDebugUIModule* CurveDebugModule = FModuleManager::LoadModulePtr<ILiveLinkCurveDebugUIModule>("LiveLinkCurveDebugUI");
	if (nullptr == CurveDebugModule)
	{
		PrintErrorToScreen(NoValidARCurveDebugModule_Warning);
	}

	return CurveDebugModule;
}

void ULiveLinkDebuggerBlueprintLibrary::DisplayLiveLinkDebugger(FString SubjectName)
{
	ILiveLinkCurveDebugUIModule* CurveDebugModule = GetLiveLinkDebugModule();
	if (nullptr != CurveDebugModule)
	{
		CurveDebugModule->DisplayLiveLinkCurveDebugUI(SubjectName);
	}
}

void ULiveLinkDebuggerBlueprintLibrary::HideLiveLinkDebugger()
{
	ILiveLinkCurveDebugUIModule* CurveDebugModule = GetLiveLinkDebugModule();
	if (nullptr != CurveDebugModule)
	{
		CurveDebugModule->HideLiveLinkCurveDebugUI();
	}
}
