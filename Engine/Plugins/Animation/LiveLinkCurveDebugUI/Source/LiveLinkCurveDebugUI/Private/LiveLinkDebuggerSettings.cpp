// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkDebuggerSettings.h"

ULiveLinkDebuggerSettings::ULiveLinkDebuggerSettings()
	: MinBarColor(FLinearColor(.05f, 0, 0, 1))
	, MaxBarColor(FLinearColor(1.f, 0, 0, 1))
	, DPIScaleMultiplier(2.5f)
{
}

//This is a really rough calculation right now, but we definitely need to consider fixing it to sue a configurable curve down the line. Should be fine for such a simple
//widget for now though
float ULiveLinkDebuggerSettings::GetDPIScaleBasedOnSize(FIntPoint Size) const
{
	//Everything was roughly laid out in 1440p, so lets compare against that vertical space
	const float AuthoredSize = 1440;

	//This is a vertical menu, so for now base all scaling off the vertical axis
	float EvalFloat = (float)Size.Y;

	//For now roughly base this on the difference between authored height and viewport height with some user-set multiplier
	return ((EvalFloat / AuthoredSize) * DPIScaleMultiplier);
}

FSlateColor ULiveLinkDebuggerSettings::GetBarColorForCurveValue(float CurveValue) const
{
	FLinearColor MinColor = MinBarColor.GetSpecifiedColor();
	FLinearColor MaxColor = MaxBarColor.GetSpecifiedColor();

	FLinearColor LerpedColor = FMath::Lerp(MinColor, MaxColor, CurveValue);

	return FSlateColor(LerpedColor);
}
