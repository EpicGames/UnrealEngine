// Copyright Epic Games, Inc. All Rights Reserved.


#include "SLiveLinkCurveDebugUI.h"
#include "Framework/Views/TableViewMetadata.h"
#include "LiveLinkCurveDebugPrivate.h"

#include "Features/IModularFeatures.h"
#include "ILiveLinkClient.h"
#include "LiveLinkDebugCurveNodeBase.h"
#include "Roles/LiveLinkBasicRole.h"
#include "SLiveLinkCurveDebugUIListItem.h"
#include "Framework/Application/SlateApplication.h"

#include "Widgets/Layout/SDPIScaler.h"
#include "Widgets/Layout/SSafeZone.h"
#include "Widgets/Views/SListView.h"


#define LOCTEXT_NAMESPACE "SLiveLinkCurveDebugUI"

void SLiveLinkCurveDebugUI::Construct(const FArguments& InArgs)
{
	this->UpdateRate = InArgs._UpdateRate;
	this->OnSubjectNameChanged = InArgs._OnSubjectNameChanged;

	CachedLiveLinkSubjectName = InArgs._InitialLiveLinkSubjectName;

	//Try and get the LiveLink Client now and cache it off
	CachedLiveLinkClient = nullptr;
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	if (ModularFeatures.IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
	{
		CachedLiveLinkClient = &IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
		ensureAlwaysMsgf((nullptr != CachedLiveLinkClient), TEXT("No valid LiveLinkClient when trying to use a SLiveLinkCurveDebugUI! LiveLinkCurveDebugUI requires LiveLinkClient!"));
	}

	const EVisibility LiveLinkSubjectHeaderVis = InArgs._ShowLiveLinkSubjectNameHeader ? EVisibility::Visible : EVisibility::Collapsed;

	SUserWidget::Construct(SUserWidget::FArguments()
	[
		SNew(SDPIScaler)
		.DPIScale(InArgs._DPIScale)
		[
			SNew(SSafeZone)
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.VAlign(VAlign_Top)
				.HAlign(HAlign_Fill)
				.AutoHeight()
				[
					SNew(STextBlock)
						.Text(this, &SLiveLinkCurveDebugUI::GetLiveLinkSubjectNameHeader)
						.ColorAndOpacity(FLinearColor(.8f, .8f, .8f, 1.0f))
						.Font(FCoreStyle::GetDefaultFontStyle("Regular", 10))
						.Visibility(LiveLinkSubjectHeaderVis)
				]

				+SVerticalBox::Slot()
				.VAlign(VAlign_Top)
				.HAlign(HAlign_Fill)
				[
					SNew(SBorder)
					[
						SAssignNew(DebugListView, SListView<TSharedPtr<FLiveLinkDebugCurveNodeBase>>)
						.ListItemsSource(&CurveData)
						.SelectionMode(ESelectionMode::None)
						.OnGenerateRow(this, &SLiveLinkCurveDebugUI::GenerateListRow)
						.HeaderRow
						(
							SNew(SHeaderRow)

							+SHeaderRow::Column(SLiveLinkCurveDebugUIListItem::NAME_CurveName)
							.DefaultLabel(LOCTEXT("CurveName","Curve Name"))
							.FillWidth(.15f)

							+ SHeaderRow::Column(SLiveLinkCurveDebugUIListItem::NAME_CurveValue)
							.DefaultLabel(LOCTEXT("CurveValue", "Curve Value"))
							.FillWidth(.85f)
						)
					]
				]
			]
		]
	]);

	//Kick off initial CurveData generation
	UpdateCurveData();
	NextUpdateTime = static_cast<double>(UpdateRate) + FSlateApplication::Get().GetCurrentTime();
}

TSharedRef<ITableRow> SLiveLinkCurveDebugUI::GenerateListRow(TSharedPtr<FLiveLinkDebugCurveNodeBase> InItem, const TSharedRef<STableViewBase>& InOwningTable)
{
	return SNew(SLiveLinkCurveDebugUIListItem, InOwningTable)
		.CurveInfo(InItem);
}

void SLiveLinkCurveDebugUI::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (DebugListView.IsValid())
	{
		const double CurrentTime = FSlateApplication::Get().GetCurrentTime();
		if (CurrentTime > NextUpdateTime)
		{
			UpdateCurveData();
			NextUpdateTime = static_cast<double>(UpdateRate) + CurrentTime;

			DebugListView->RequestListRefresh();
		}
	}

	SUserWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

void SLiveLinkCurveDebugUI::UpdateCurveData()
{
	CurveData.Reset();

	//If we don't have a good live link subject name, lets try to get one
	if (!CachedLiveLinkSubjectName.IsValid() || CachedLiveLinkSubjectName.IsNone())
	{
		ChangeToNextValidLiveLinkSubjectName();
	}

	if (ensureMsgf((nullptr!= CachedLiveLinkClient), TEXT("No valid LiveLinkClient! Can not update curve data for LiveLinkCurveDebugUI")))
	{
		bool bDataIsValid = false;
		FLiveLinkSubjectFrameData SubjectData;
		if (CachedLiveLinkClient->EvaluateFrame_AnyThread(CachedLiveLinkSubjectName, ULiveLinkBasicRole::StaticClass(), SubjectData))
		{
			FLiveLinkBaseStaticData* StaticData = SubjectData.StaticData.Cast<FLiveLinkBaseStaticData>();
			FLiveLinkBaseFrameData* FrameData = SubjectData.FrameData.Cast<FLiveLinkBaseFrameData>();

			if (StaticData && FrameData && StaticData->PropertyNames.Num() && FrameData->PropertyValues.Num() == StaticData->PropertyNames.Num())
			{
				bDataIsValid = true;

				int32 PropertyNum = StaticData->PropertyNames.Num();
				for (int CurveIndex = 0; CurveIndex < PropertyNum; ++CurveIndex)
				{
					CurveData.Add(MakeShared<FLiveLinkDebugCurveNodeBase>(StaticData->PropertyNames[CurveIndex], FrameData->PropertyValues[CurveIndex]));
				}
			}
		}

		//Just show an error curve message until we have a frame for the client.
		if (!bDataIsValid)
		{
			CurveData.Reset();

			const FText NoCurvesText = LOCTEXT("NoCurvesForSubject", "No Curve Data");
			CurveData.Add(MakeShareable<FLiveLinkDebugCurveNodeBase>(new FLiveLinkDebugCurveNodeBase(*NoCurvesText.ToString(), 0.0f)));
		}
	}
}

void SLiveLinkCurveDebugUI::ChangeToNextValidLiveLinkSubjectName()
{
	if (nullptr != CachedLiveLinkClient)
	{
		TArray<FLiveLinkSubjectKey> ActiveSubjects = CachedLiveLinkClient->GetSubjects(false, true);
		int32 FoundIndex = ActiveSubjects.IndexOfByPredicate([this](const FLiveLinkSubjectKey& Other) { return Other.SubjectName == CachedLiveLinkSubjectName; });
		int32 NewIndex = ActiveSubjects.IsValidIndex(FoundIndex + 1) ? FoundIndex + 1 : 0;
		if (ActiveSubjects.IsValidIndex(NewIndex))
		{
			SetLiveLinkSubjectName(ActiveSubjects[NewIndex].SubjectName);
		}
	}
}

void SLiveLinkCurveDebugUI::GetAllSubjectNames(TArray<FName>& OutSubjectNames) const
{
	OutSubjectNames.Reset();

	if (ensureAlwaysMsgf(CachedLiveLinkClient, TEXT("No valid CachedLiveLinkClient when attempting to use SLiveLinkCurveDebugUI::GetAllSubjectNames! The SLiveLinkCurveDebugUI should always have a cached live link client!")))
	{
		TArray<FLiveLinkSubjectKey> ActiveSubjects = CachedLiveLinkClient->GetSubjects(false, true);
		OutSubjectNames.Reset(OutSubjectNames.Num());
		for (const FLiveLinkSubjectKey& SubjectKey : ActiveSubjects)
		{
			OutSubjectNames.Add(SubjectKey.SubjectName);
		}
	}
}

void SLiveLinkCurveDebugUI::SetLiveLinkSubjectName(FName SubjectName)
{
	if (SubjectName != CachedLiveLinkSubjectName)
	{
		CachedLiveLinkSubjectName = SubjectName;
		UE_LOGF(LogLiveLinkCurveDebugUI, Display, "Set LiveLinkSubjectName: %ls", *CachedLiveLinkSubjectName.ToString());

		//Update next tick
		NextUpdateTime = FSlateApplication::Get().GetCurrentTime();

		if (OnSubjectNameChanged.IsBound())
		{
			OnSubjectNameChanged.Execute(CachedLiveLinkSubjectName);
		}
	}
}

FText SLiveLinkCurveDebugUI::GetLiveLinkSubjectNameHeader() const
{
	return FText::Format(LOCTEXT("LiveLinkSubjectNameHeader", "Currently Viewing: {0}"), FText::FromName(CachedLiveLinkSubjectName));
}

TSharedRef<SLiveLinkCurveDebugUI> SLiveLinkCurveDebugUI::New()
{
	return MakeShareable(new SLiveLinkCurveDebugUI());
}

#undef LOCTEXT_NAMESPACE
