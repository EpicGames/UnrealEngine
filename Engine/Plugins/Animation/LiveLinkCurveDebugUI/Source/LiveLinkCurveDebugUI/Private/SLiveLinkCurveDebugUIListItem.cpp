// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLiveLinkCurveDebugUIListItem.h"

#include "Brushes/SlateColorBrush.h"
#include "LiveLinkDebugCurveNodeBase.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Widgets/Views/SListView.h"


FName SLiveLinkCurveDebugUIListItem::NAME_CurveName(TEXT("CurveName"));
FName SLiveLinkCurveDebugUIListItem::NAME_CurveValue(TEXT("CurveValue"));

void SLiveLinkCurveDebugUIListItem::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	this->CurveInfo = InArgs._CurveInfo;
	this->SetPadding(0);

	ensureAlwaysMsgf(CurveInfo.IsValid(), TEXT("Attempt to create SLiveLinkCurveDebugUIListItem with invalid CurveInfo!"));

	SMultiColumnTableRow<TSharedPtr<FLiveLinkDebugCurveNodeBase>>::Construct(SMultiColumnTableRow<TSharedPtr<FLiveLinkDebugCurveNodeBase>>::FArguments().Padding(0), InOwnerTableView);
}

TSharedRef<SWidget> SLiveLinkCurveDebugUIListItem::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (ColumnName == NAME_CurveName)
	{
		return SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0.0f, 0.0f)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
				.Text(this, &SLiveLinkCurveDebugUIListItem::GetCurveName)
				.ColorAndOpacity(FLinearColor(.8f,.8f,.8f,1.0f))
				.Font(FCoreStyle::GetDefaultFontStyle("Regular", 10))
		];
	}
	else if (ColumnName == NAME_CurveValue)
	{
		return SNew(SBox)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.Padding(FMargin(0.0f, 0.0f))
		[
			SNew(SProgressBar)
			.Percent(this, &SLiveLinkCurveDebugUIListItem::GetCurveValue)
			.FillColorAndOpacity(this, &SLiveLinkCurveDebugUIListItem::GetProgressFillColor)
			.BackgroundImage(GetProgressBackgroundImage())
		];
	}
	else
	{
		return SNullWidget::NullWidget;
	}
}

FText SLiveLinkCurveDebugUIListItem::GetCurveName() const
{
	if (ensureAlwaysMsgf(CurveInfo.IsValid(), TEXT("Invalid Curve Info in the LiveLinkCurveDebugUI! This means a SLiveLinkCurveDebugUIListItem widget was created without valid data!")))
	{
		return CurveInfo->GetCurveName();
	}

	return FText();
}

TOptional<float> SLiveLinkCurveDebugUIListItem::GetCurveValue() const
{
	if (ensureAlwaysMsgf(CurveInfo.IsValid(), TEXT("Invalid Curve Info in the LiveLinkCurveDebugUI! This means a SLiveLinkCurveDebugUIListItem widget was created without valid data!")))
	{
		TOptional<float> OpFloat;
		OpFloat.Emplace(CurveInfo->GetCurveValue());
		return OpFloat;
	}

	return TOptional<float>();
}

FSlateColor SLiveLinkCurveDebugUIListItem::GetProgressFillColor() const
{
	if (ensureAlwaysMsgf(CurveInfo.IsValid(), TEXT("Invalid Curve Info in the LiveLinkCurveDebugUI! This means a SLiveLinkCurveDebugUIListItem widget was created without valid data!")))
	{
		return CurveInfo->GetCurveFillColor();
	}

	return FSlateColor();
}

const FSlateBrush* SLiveLinkCurveDebugUIListItem::GetProgressBackgroundImage() const
{
	// For now use a transparent background for progress bar
	static FSlateColorBrush ColorBrush(FLinearColor(0, 0, 0, 0));
	return &ColorBrush;
}
