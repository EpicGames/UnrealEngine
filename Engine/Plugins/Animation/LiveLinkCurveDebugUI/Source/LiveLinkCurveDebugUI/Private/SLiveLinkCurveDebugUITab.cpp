// Copyright Epic Games, Inc. All Rights Reserved.


#include "SLiveLinkCurveDebugUITab.h"
#include "Framework/Views/TableViewMetadata.h"
#include "SLiveLinkCurveDebugUI.h"

#include "Widgets/Input/SComboBox.h"


#define LOCTEXT_NAMESPACE "SLiveLinkCurveDebugUI"

void SLiveLinkCurveDebugUITab::Construct(const FArguments& InArgs)
{
	SDockTab::Construct( SDockTab::FArguments()
		.TabRole(ETabRole::NomadTab)
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Top)
		.AutoHeight()
		.Padding(5,5)
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("LiveLinkSubjectName","Live Link Subject Name:"))
				.Font(FCoreStyle::GetDefaultFontStyle("Regular",12))
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SAssignNew(SubjectNameComboBox, SComboBox<TSharedPtr<FString>>)
				.OptionsSource(&SubjectNames)
				.OnGenerateWidget(this, &SLiveLinkCurveDebugUITab::MakeComboButtonItemWidget)
				.OnSelectionChanged(this, &SLiveLinkCurveDebugUITab::OnSelectionChanged)
				.OnComboBoxOpening(this, &SLiveLinkCurveDebugUITab::OnSubjectNameComboBoxOpened)
				[
					SNew(STextBlock)
					.Text(this, &SLiveLinkCurveDebugUITab::GetSelectedSubjectName)
				]
			]
		]

		+ SVerticalBox::Slot()
		.VAlign(VAlign_Fill)
		.Padding(5,5)
		[
			SAssignNew(MyDebugUI, SLiveLinkCurveDebugUI)
				.InitialLiveLinkSubjectName(InArgs._InitialLiveLinkSubjectName)
				.OnSubjectNameChanged(this, &SLiveLinkCurveDebugUITab::UpdateSubjectNameEditor)
				.DPIScale(1.0f) // For the tab we are just using seperate 1.0 DPI as its a seperate window
				.ShowLiveLinkSubjectNameHeader(false) // Don't want to show the header as we already show this information in our above SubjectName box
		]
	]);

	SetLiveLinkSubjectName(InArgs._InitialLiveLinkSubjectName);
}

void SLiveLinkCurveDebugUITab::UpdateSubjectNameEditor(FName SubjectName)
{
	CurrentSelectedSubjectName = MakeShareable(new FString(SubjectName.ToString()));
	RefreshSubjectNames();
}

void SLiveLinkCurveDebugUITab::SetLiveLinkSubjectName(FName SubjectName)
{
	CurrentSelectedSubjectName = MakeShareable(new FString(SubjectName.ToString()));

	if (MyDebugUI.IsValid())
	{
		MyDebugUI->SetLiveLinkSubjectName(SubjectName);
	}
}

void SLiveLinkCurveDebugUITab::OnSelectionChanged(TSharedPtr<FString> StringItem, ESelectInfo::Type SelectInfo)
{
	if (StringItem.IsValid())
	{
		SetLiveLinkSubjectName(FName(**StringItem));
	}
}

TSharedRef<SWidget> SLiveLinkCurveDebugUITab::MakeComboButtonItemWidget(TSharedPtr<FString> StringItem)
{
	return SNew(STextBlock)
		.Text(FText::FromString(*StringItem));
}

FText SLiveLinkCurveDebugUITab::GetSelectedSubjectName() const
{
	return CurrentSelectedSubjectName.IsValid() ? FText::FromString(*CurrentSelectedSubjectName) : FText();
}

void SLiveLinkCurveDebugUITab::OnSubjectNameComboBoxOpened()
{
	RefreshSubjectNames();
}

void SLiveLinkCurveDebugUITab::RefreshSubjectNames()
{
	//Refresh our list of SubjectNames
	SubjectNames.Reset();

	if (MyDebugUI.IsValid())
	{
		TArray<FName> GatheredSubjectNames;
		MyDebugUI->GetAllSubjectNames(GatheredSubjectNames);

		for (FName SubjectName : GatheredSubjectNames)
		{
			SubjectNames.Add(MakeShareable(new FString(SubjectName.ToString())));
		}
	}

	//Always make sure our current one is in the list
	if (!SubjectNames.Find(CurrentSelectedSubjectName))
	{
		SubjectNames.Add(CurrentSelectedSubjectName);
	}

	if (SubjectNameComboBox.IsValid())
	{
		SubjectNameComboBox->RefreshOptions();
	}
}


#undef LOCTEXT_NAMESPACE
