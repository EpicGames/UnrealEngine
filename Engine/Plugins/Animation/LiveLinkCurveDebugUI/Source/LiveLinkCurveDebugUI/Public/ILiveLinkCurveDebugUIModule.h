// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

/**
 * Interface for messaging modules.
 */
class ILiveLinkCurveDebugUIModule
	: public IModuleInterface
{
public:

	/**
	 * Display the LiveLinkCurveDebugUI, either spawned from a tab manager, or as a custom widget attached to our current viewport if the tab manager can not be used
	 */
	virtual void DisplayLiveLinkCurveDebugUI(FString& LiveLinkSubjectName) = 0;

	//virtual void DisplayLiveLinkCurveDebugUIWithSpecificCurveList(FString& LiveLinkSubjectName, TArray<FString>& CurveNameDebugList) = 0;

	virtual void HideLiveLinkCurveDebugUI() = 0;

	/**
	 * Registers a tab spawner for the LiveLinkCurveDebugUI.
	 *
	 * @param WorkspaceGroup The workspace group to insert the tab into.
	 */
	virtual void RegisterTabSpawner() = 0;

	/** Unregisters the tab spawner for the LiveLinkCurveDebugUI. */
	virtual void UnregisterTabSpawner() = 0;

public:

	/** Virtual destructor. */
	virtual ~ILiveLinkCurveDebugUIModule() { }
};
