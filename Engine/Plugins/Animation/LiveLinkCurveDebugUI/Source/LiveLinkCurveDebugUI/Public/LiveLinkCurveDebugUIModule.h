// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "ILiveLinkCurveDebugUIModule.h"

class SLiveLinkCurveDebugUI;

class SDockTab;
class FSpawnTabArgs;
class UGameViewportClient;

class FLiveLinkCurveDebugUIModule
	: public FSelfRegisteringExec
	, public ILiveLinkCurveDebugUIModule
{
public:
	// FSelfRegisteringExec interface
	virtual bool Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;

	// IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	// Functions to control UI
	virtual void DisplayLiveLinkCurveDebugUI(FString& LiveLinkSubjectName) override;
	virtual void HideLiveLinkCurveDebugUI() override;

private:
	// Our hooks into the Tab Spawn system
	virtual void RegisterTabSpawner() override;
	virtual void UnregisterTabSpawner() override;

	void RegisterSettings();
	void UnRegisterSettings();

	//Functions that do the real work of displaying the UI appropriately
	void DisplayThroughTab();
	void DisplayThroughViewportAdd();

	void SwitchToNextLiveLinkSubject();

	//Functions to create what is needed internally
	TSharedPtr<SLiveLinkCurveDebugUI> CreateDebugWidget();
	TSharedRef<SDockTab> MakeLiveLinkCurveDebugTab(const FSpawnTabArgs& Args);

	//Needed so we can spawn in a UUserWidget in this world
	UWorld* GetWorldForDebugUIModule();
    UGameViewportClient* GetGameViewportClientForDebugUIModule();
    
	//Adds/Removes our stored LiveLinkUserWidget to the viewport of the first local player
	bool AddWidgetToViewport();
	void RemoveWidgetFromViewport();

	float GetDPIScaleFromSettings();

private:
	bool bForceDisplayThroughViewport;
	bool bHasRegisteredTabSpawners;

	//A cached version of what the SubjectName to supply to LiveLink. Gets passed into created widgets
	FString LiveLinkSubjectNameToTrack;

	TSharedPtr<SLiveLinkCurveDebugUI> LiveLinkUserWidget;
};
