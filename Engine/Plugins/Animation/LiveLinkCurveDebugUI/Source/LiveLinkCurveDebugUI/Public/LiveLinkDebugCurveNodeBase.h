// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FSlateColor;

class FLiveLinkDebugCurveNodeBase
{
public:
	FLiveLinkDebugCurveNodeBase(FName InCurveName, float InCurveValue);
	virtual ~FLiveLinkDebugCurveNodeBase() {}

	//Name of the curve we are tracking
	virtual FText GetCurveName() const { return FText::FromName(CurveName); }

	//Value of the curve we are tracking
	virtual float GetCurveValue() const { return CurveValue; }

	//Color we should set the progress fill bar to be when rendering debug info for this curve
	virtual FSlateColor GetCurveFillColor() const;

protected:
	FName CurveName;
	float CurveValue;
};