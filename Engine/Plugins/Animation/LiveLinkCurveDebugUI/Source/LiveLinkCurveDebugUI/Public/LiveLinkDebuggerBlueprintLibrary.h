// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "LiveLinkDebuggerBlueprintLibrary.generated.h"

UCLASS()
class ULiveLinkDebuggerBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

	UFUNCTION(BlueprintCallable, Category = "LiveLinkDebug")
	static void DisplayLiveLinkDebugger(FString SubjectName);

	UFUNCTION(BlueprintCallable, Category = "LiveLinkDebug")
	static void HideLiveLinkDebugger();
};