// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"

#include "Styling/SlateColor.h"

#include "LiveLinkDebuggerSettings.generated.h"

class ULiveLinkCurveDebugUI;

UCLASS(config = Engine, defaultconfig)
class ULiveLinkDebuggerSettings : public UObject
{
public:

	GENERATED_BODY()

	ULiveLinkDebuggerSettings();

	//TODO : Make DPI based off a nice curve at some point, for now we just do a rough calculation
	//UPROPERTY(config, EditAnywhere, Category = "LiveLinkDebugger", meta = (
	//	DisplayName = "DPI Scaling Settings",
	//	XAxisName = "Resolution",
	//	YAxisName = "Scale"))
	//FRuntimeFloatCurve DebuggerUIScaleCurve;

	//Color used when the CurveValue bar is at 0
	UPROPERTY(config, EditAnywhere, Category = "LiveLinkDebugger", meta= ( DisplayName = "Minimum Bar Color Value"))
	FSlateColor MinBarColor;

	//Color used when the CurveValueBar is at 1.0
	UPROPERTY(config, EditAnywhere, Category = "LiveLinkDebugger", meta = (DisplayName = "Maximum Bar Color Value"))
	FSlateColor MaxBarColor;

	//This multiplier is used on the Viewport Widget version (IE: In Game) as it needs to be slightly more aggresive then the PC version
	UPROPERTY(config, EditAnywhere, Category = "LiveLinkDebugger", meta = (DisplayName = "DPI Scale Multiplier"))
	float DPIScaleMultiplier;

	UFUNCTION(BlueprintCallable, Category = "LiveLinkDebugger")
	float GetDPIScaleBasedOnSize(FIntPoint Size) const;

	UFUNCTION(BlueprintCallable, Category = "LiveLinkDebugger")
	FSlateColor GetBarColorForCurveValue(float CurveValue) const;
};