// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SUserWidget.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableViewBase.h"
#include "LiveLinkDebugCurveNodeBase.h"

class SLiveLinkCurveDebugUI
	: public SUserWidget
{
public:

	//Delegate used to call back when this widget sets its own SubjectName from the LiveLink client instead of any supplied InitialLiveLinkSubjectName
	DECLARE_DELEGATE_OneParam(FOnSubjectNameChanged, FName);

	SLATE_USER_ARGS(SLiveLinkCurveDebugUI)
		: _DPIScale(1.0f)
		, _InitialLiveLinkSubjectName(FName())
		, _UpdateRate(0.0f)
		, _ShowLiveLinkSubjectNameHeader(true)
	{}
		SLATE_ARGUMENT(float, DPIScale)
		SLATE_ARGUMENT(FName, InitialLiveLinkSubjectName)
		SLATE_ARGUMENT(float, UpdateRate)
		SLATE_ARGUMENT(bool, ShowLiveLinkSubjectNameHeader)

		SLATE_EVENT(FOnSubjectNameChanged, OnSubjectNameChanged)

	SLATE_END_ARGS()

	virtual void Construct(const FArguments& InArgs);

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	virtual void SetLiveLinkSubjectName(FName SubjectName);

	//Gets all current LiveLink subject names from our internal LiveLink client and stores it in the OutSubjectNames parameter. Clears OutSubjectNames if no SubjectNames are found!
	virtual void GetAllSubjectNames(TArray<FName>& OutSubjectNames) const;

	//If we don't have a suplied CachedLiveLinkSubjectName, we try to pull it from LiveLink by looking at
	//All available subjects and picking the first one we find with curves to supply
	virtual void ChangeToNextValidLiveLinkSubjectName();

private:
	FText GetLiveLinkSubjectNameHeader() const;

	void UpdateCurveData();
	
	// Called by the list view to generate a row widget from the supplied data
	TSharedRef<ITableRow> GenerateListRow(TSharedPtr<FLiveLinkDebugCurveNodeBase> InItem, const TSharedRef<STableViewBase>& InOwningTable);

	//Cached information to generate LiveLinkData	
	FName CachedLiveLinkSubjectName;
	class ILiveLinkClient* CachedLiveLinkClient;

	//Actual generated data being used to generate items in the ListView
	TArray<TSharedPtr<FLiveLinkDebugCurveNodeBase>> CurveData;

	//Callback we use if our widget updates its SubjectName
	FOnSubjectNameChanged OnSubjectNameChanged;

	//Used to limit how often we update curve data for performance
	float UpdateRate;
	double NextQueuedUpdateTime;
	double NextUpdateTime;

	TSharedPtr<SListView<TSharedPtr<FLiveLinkDebugCurveNodeBase>>> DebugListView;
};

