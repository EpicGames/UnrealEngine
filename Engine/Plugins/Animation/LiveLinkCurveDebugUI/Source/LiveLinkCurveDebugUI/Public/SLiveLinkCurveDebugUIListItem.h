// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"

#include "LiveLinkDebugCurveNodeBase.h"

class SLiveLinkCurveDebugUIListItem
	: public SMultiColumnTableRow<TSharedPtr<FLiveLinkDebugCurveNodeBase>>
{
public:
	static FName NAME_CurveName;
	static FName NAME_CurveValue;

	SLATE_BEGIN_ARGS(SLiveLinkCurveDebugUIListItem)
		: _CurveInfo()
	{ }

		SLATE_ARGUMENT(TSharedPtr<FLiveLinkDebugCurveNodeBase>, CurveInfo)

	SLATE_END_ARGS()

public:

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView);

	virtual FText GetCurveName() const;

	virtual TOptional<float> GetCurveValue() const;

	virtual FSlateColor GetProgressFillColor() const;

	const FSlateBrush* GetProgressBackgroundImage() const;

	// SMultiColumnTableRow overrides
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

private:

	TSharedPtr<FLiveLinkDebugCurveNodeBase> CurveInfo;
};