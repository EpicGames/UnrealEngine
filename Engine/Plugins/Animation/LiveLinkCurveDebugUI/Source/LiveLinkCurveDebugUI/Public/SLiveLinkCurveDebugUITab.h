// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SComboBox.h"

class SLiveLinkCurveDebugUI;
class SEditableTextBox;

class SLiveLinkCurveDebugUITab
	: public SDockTab
{
public:

	SLATE_BEGIN_ARGS(SLiveLinkCurveDebugUITab)
	{
	}
		SLATE_ARGUMENT(FName, InitialLiveLinkSubjectName)

	SLATE_END_ARGS()

	virtual void Construct(const FArguments& InArgs);
	void SetLiveLinkSubjectName(FName SubjectName);
	
protected:
	TSharedRef<SWidget> MakeComboButtonItemWidget(TSharedPtr<FString> StringItem);
	void OnSelectionChanged(TSharedPtr<FString> StringItem, ESelectInfo::Type SelectInfo);
	FText GetSelectedSubjectName() const;
	void OnSubjectNameComboBoxOpened();

	void RefreshSubjectNames();

	//This is for when our subject name is updated by the underlying widget and we need to change our selected SubjectName
	void UpdateSubjectNameEditor(FName SubjectName);

	/* Our embedded LiveLink Curve Debugger */
	TSharedPtr<SLiveLinkCurveDebugUI> MyDebugUI;

	TSharedPtr<SComboBox<TSharedPtr<FString>>> SubjectNameComboBox;

	TSharedPtr<FString> CurrentSelectedSubjectName;
	TArray<TSharedPtr<FString>> SubjectNames;
};

